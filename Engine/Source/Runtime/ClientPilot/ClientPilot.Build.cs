// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ClientPilot : ModuleRules
{
    public ClientPilot(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
            }
        );

		PublicIncludePathModuleNames.AddRange(new string[] {
				"AutomationController",
				"AutomationTest",
		});

		if (Target.bCompileAgainstEngine && Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			DynamicallyLoadedModuleNames.AddRange(new string[] { "AutomationController" });
		}
	}
}
