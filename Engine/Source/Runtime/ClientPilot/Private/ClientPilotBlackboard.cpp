// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientPilotBlackboard.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClientPilotBlackboard)

void UClientPilotBlackboard::InitializeFromProfile(FString ProfileCategoryAndName)
{

}

FString UClientPilotBlackboard::GetStringValue(FString KeyName)
{
	FString RetVal;
	if (Blackboard.Contains(KeyName))
	{
		RetVal = Blackboard[KeyName];
	}
	return RetVal;
}

FVector UClientPilotBlackboard::GetVectorValue(FString KeyName)
{
	FVector RetVal = FVector::ZeroVector;
	if (Blackboard.Contains(KeyName))
	{
		RetVal.InitFromString(*Blackboard[KeyName]);
	}
	return RetVal;
}

float UClientPilotBlackboard::GetFloatValue(FString KeyName)
{
	float RetVal = 0.f;
	if (Blackboard.Contains(KeyName))
	{
		RetVal = FCString::Atof(*Blackboard[KeyName]);
	}
	return RetVal;
}

int UClientPilotBlackboard::GetIntValue(FString KeyName)
{
	int RetVal = 0;
	if (Blackboard.Contains(KeyName))
	{
		RetVal = FCString::Atoi(*Blackboard[KeyName]);
	}
	return RetVal;
}

void UClientPilotBlackboard::AddOrUpdateValue(FString KeyName, float Value)
{
	Blackboard.Emplace(KeyName, FString::SanitizeFloat(Value));
}

void UClientPilotBlackboard::AddOrUpdateValue(FString KeyName, int Value)
{
	Blackboard.Emplace(KeyName, FString::FromInt(Value));
}

void UClientPilotBlackboard::AddOrUpdateValue(FString KeyName, FString Value)
{
	Blackboard.Emplace(KeyName, Value);
}

void UClientPilotBlackboard::AddOrUpdateValue(FString KeyName, FVector Value)
{
	Blackboard.Emplace(KeyName, Value.ToString());
}
