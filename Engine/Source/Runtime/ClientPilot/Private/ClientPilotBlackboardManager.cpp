// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientPilotBlackboardManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClientPilotBlackboardManager)

UClientPilotBlackboardManager* UClientPilotBlackboardManager::ObjectInstance = nullptr;

UClientPilotBlackboardManager* UClientPilotBlackboardManager::GetInstance()
{
#if !UE_BUILD_SHIPPING
	if (ObjectInstance == nullptr)
	{
		ObjectInstance = NewObject<UClientPilotBlackboardManager>();
		ObjectInstance->AddToRoot();
	}
#endif
	return ObjectInstance;
}
