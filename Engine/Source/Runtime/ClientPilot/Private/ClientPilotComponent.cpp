// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientPilotComponent.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClientPilotComponent)


UClientPilotComponent::UClientPilotComponent()
{
}

UClientPilotBlackboard* UClientPilotComponent::GetBlackboardInstance()
{
	return UClientPilotBlackboardManager::GetInstance() ? UClientPilotBlackboardManager::GetInstance()->PilotBlackboard : nullptr;
}

void UClientPilotComponent::ThinkAndAct() 
{
}

IMPLEMENT_MODULE(FDefaultModuleImpl, ClientPilot);
