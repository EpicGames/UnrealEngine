// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Math/MathFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClientPilotBlackboard.generated.h"

UCLASS(MinimalAPI)
class UClientPilotBlackboard : public UObject
{
GENERATED_BODY()
public:
	CLIENTPILOT_API virtual void InitializeFromProfile(FString CategoryToUse);

	CLIENTPILOT_API void AddOrUpdateValue(FString KeyName, float Value);
	CLIENTPILOT_API void AddOrUpdateValue(FString KeyName, int Value);
	CLIENTPILOT_API void AddOrUpdateValue(FString KeyName, FString Value);
	CLIENTPILOT_API void AddOrUpdateValue(FString KeyName, FVector Value);

	CLIENTPILOT_API FString GetStringValue(FString KeyName);
	CLIENTPILOT_API int GetIntValue(FString KeyName);
	CLIENTPILOT_API float GetFloatValue(FString KeyName);
	CLIENTPILOT_API FVector GetVectorValue(FString KeyName);

	void RemoveKey(FString Key) 
	{ 
		Blackboard.Remove(Key);
	}
	void ResetBlackboard()
	{
		Blackboard.Empty();
	}

protected:
	TMap<FString, FString> Blackboard;
};
