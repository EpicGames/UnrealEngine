// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClientPilotBlackboard.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ClientPilotBlackboardManager.generated.h"

class UClientPilotBlackboard;


UCLASS(MinimalAPI)
class UClientPilotBlackboardManager : public UObject
{
GENERATED_BODY()
protected:
	static CLIENTPILOT_API UClientPilotBlackboardManager * ObjectInstance;

public:
	static CLIENTPILOT_API UClientPilotBlackboardManager * GetInstance();
	UPROPERTY()
	TObjectPtr<UClientPilotBlackboard> PilotBlackboard;
};
