// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ClientPilotBlackboardManager.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ClientPilotComponent.generated.h"

class UClientPilotBlackboard;

UCLASS(MinimalAPI)
class UClientPilotComponent : public UObject
{
	GENERATED_BODY()
public:
	CLIENTPILOT_API UClientPilotComponent();
	CLIENTPILOT_API UClientPilotBlackboard* GetBlackboardInstance();
	CLIENTPILOT_API virtual void ThinkAndAct();

};
